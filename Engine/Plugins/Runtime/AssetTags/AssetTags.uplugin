{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Asset Tags",
	"Description" : "Provides high-level management and access to asset tags and collections for runtime and editor scripting.",
	"Category" : "Other",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AssetTags",
			"Type" : "Runtime",
			"LoadingPhase": "Default"
 		}
	]
}
