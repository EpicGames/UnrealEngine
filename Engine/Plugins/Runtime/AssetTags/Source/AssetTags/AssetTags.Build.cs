// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AssetTags : ModuleRules
	{
		public AssetTags(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[] {
					"AssetRegistry",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetRegistry",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"CollectionManager",
						"UnrealEd"
					}
				);
			}
		}
	}
}
