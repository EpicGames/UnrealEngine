// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTagsSubsystem.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetTagsSubsystem)

#if WITH_EDITOR
#include "CollectionManagerModule.h"
#include "Editor/UnrealEdEngine.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "Subsystems/CollectionManagerScriptingSubsystem.h"
#endif	// WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogAssetTags, Log, All);

#if WITH_EDITOR

extern UNREALED_API UEditorEngine* GEditor;

namespace AssetTagsSubsystemUtil
{

ECollectionScriptingShareType ECollectionShareType_To_ECollectionScriptingShareType(const ECollectionShareType::Type ShareType)
{
	static_assert((int32)ECollectionShareType::CST_Local == (int32)ECollectionScriptingShareType::Local + 1, "ECollectionShareType::CST_Local is expected to be ECollectionScriptingShareType::Local + 1");
	static_assert((int32)ECollectionShareType::CST_Private == (int32)ECollectionScriptingShareType::Private + 1, "ECollectionShareType::CST_Private is expected to be ECollectionScriptingShareType::Private + 1");
	static_assert((int32)ECollectionShareType::CST_Shared == (int32)ECollectionScriptingShareType::Shared + 1, "ECollectionShareType::CST_Shared is expected to be ECollectionScriptingShareType::Shared + 1");

	return (ECollectionScriptingShareType)((int32)ShareType - 1);
}

bool FindCollectionByName(ICollectionContainer& CollectionContainer, const FName Name, FCollectionScriptingRef& OutCollection)
{
	TArray<FCollectionNameType> CollectionNamesAndTypes;
	CollectionContainer.GetCollections(Name, CollectionNamesAndTypes);
	
	if (CollectionNamesAndTypes.Num() == 0)
	{
		UE_LOGF(LogAssetTags, Warning, "No collection found called '%ls'", *Name.ToString());
		return false;
	}
	else if (CollectionNamesAndTypes.Num() > 1)
	{
		UE_LOGF(LogAssetTags, Warning, "%d collections found called '%ls'; ambiguous result", CollectionNamesAndTypes.Num(), *Name.ToString());
		return false;
	}

	OutCollection = { .Container = CollectionContainer.GetCollectionSource()->GetName(), 
		.Name = CollectionNamesAndTypes[0].Name, .ShareType = AssetTagsSubsystemUtil::ECollectionShareType_To_ECollectionScriptingShareType(CollectionNamesAndTypes[0].Type) };

	return true;
}

}	// namespace AssetTagsSubsystemUtil

bool UAssetTagsSubsystem::CreateCollection(const FName Name, const ECollectionScriptingShareType ShareType)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();
		FCollectionScriptingRef NewCollection;
		return Subsystem->CreateCollection(FCollectionScriptingContainerSource{ CollectionContainer->GetCollectionSource()->GetName(), CollectionContainer->GetCollectionSource()->GetTitle() }, Name, ShareType, NewCollection);
	}
	
	return false;
}

bool UAssetTagsSubsystem::DestroyCollection(const FName Name)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();
		FCollectionScriptingRef Collection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, Collection))
		{
			return Subsystem->DestroyCollection(Collection);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::RenameCollection(const FName Name, const FName NewName)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef Collection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, Collection))
		{
			return Subsystem->RenameCollection(Collection, NewName, Collection.ShareType);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::ReparentCollection(const FName Name, const FName NewParentName)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef ResolvedCollection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
		{
			FCollectionScriptingRef ResolvedParentCollection { .Container = CollectionContainer->GetCollectionSource()->GetName() };
			if (NewParentName.IsNone() || AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, NewParentName, ResolvedParentCollection))
			{
				return Subsystem->ReparentCollection(ResolvedCollection, ResolvedParentCollection);
			}
		}
	}

	return false;
}

bool UAssetTagsSubsystem::EmptyCollection(const FName Name)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef ResolvedCollection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
		{
			return Subsystem->EmptyCollection(ResolvedCollection);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::K2_AddAssetToCollection(const FName Name, const FSoftObjectPath& AssetPath)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef ResolvedCollection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
		{
			return Subsystem->AddAssetToCollection(ResolvedCollection, AssetPath);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::AddAssetToCollection(const FName Name, const FName AssetPathName)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_AddAssetToCollection(Name, FSoftObjectPath(AssetPathName.ToString()));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::AddAssetDataToCollection(const FName Name, const FAssetData& AssetData)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_AddAssetToCollection(Name, AssetData.GetSoftObjectPath());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::AddAssetPtrToCollection(const FName Name, const UObject* AssetPtr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_AddAssetToCollection(Name, FSoftObjectPath(AssetPtr));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::K2_AddAssetsToCollection(const FName Name, const TArray<FSoftObjectPath>& AssetPaths)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef ResolvedCollection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
		{
			return Subsystem->AddAssetsToCollection(ResolvedCollection, AssetPaths);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::AddAssetsToCollection(const FName Name, const TArray<FName>& AssetPathNames)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_AddAssetsToCollection(Name, UE::SoftObjectPath::Private::ConvertObjectPathNames(AssetPathNames));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::AddAssetDatasToCollection(const FName Name, const TArray<FAssetData>& AssetDatas)
{
	TArray<FSoftObjectPath> AssetPaths;
	AssetPaths.Reserve(AssetDatas.Num());
	for (const FAssetData& AssetData : AssetDatas)
	{
		AssetPaths.Add(AssetData.GetSoftObjectPath());
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_AddAssetsToCollection(Name, AssetPaths);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::AddAssetPtrsToCollection(const FName Name, const TArray<UObject*>& AssetPtrs)
{
	TArray<FSoftObjectPath> AssetPaths;
	AssetPaths.Reserve(AssetPtrs.Num());
	for (const UObject* AssetPtr : AssetPtrs)
	{
		AssetPaths.Add(FSoftObjectPath(AssetPtr));
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_AddAssetsToCollection(Name, AssetPaths);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::K2_RemoveAssetFromCollection(const FName Name, const FSoftObjectPath& AssetPath)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef ResolvedCollection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
		{
			return Subsystem->RemoveAssetFromCollection(ResolvedCollection, AssetPath);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::RemoveAssetFromCollection(const FName Name, const FName AssetPathName)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_RemoveAssetFromCollection(Name, FSoftObjectPath(AssetPathName.ToString()));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::RemoveAssetDataFromCollection(const FName Name, const FAssetData& AssetData)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_RemoveAssetFromCollection(Name, AssetData.GetSoftObjectPath());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::RemoveAssetPtrFromCollection(const FName Name, const UObject* AssetPtr)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_RemoveAssetFromCollection(Name, FSoftObjectPath(AssetPtr));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::K2_RemoveAssetsFromCollection(const FName Name, const TArray<FSoftObjectPath>& AssetPaths)
{
	if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
	{
		const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

		FCollectionScriptingRef ResolvedCollection;
		if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
		{
			return Subsystem->RemoveAssetsFromCollection(ResolvedCollection, AssetPaths);
		}
	}

	return false;
}

bool UAssetTagsSubsystem::RemoveAssetsFromCollection(const FName Name, const TArray<FName>& AssetPathNames)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_RemoveAssetsFromCollection(Name, UE::SoftObjectPath::Private::ConvertObjectPathNames(AssetPathNames));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::RemoveAssetDatasFromCollection(const FName Name, const TArray<FAssetData>& AssetDatas)
{
	TArray<FSoftObjectPath> AssetPaths;
	AssetPaths.Reserve(AssetDatas.Num());
	for (const FAssetData& AssetData : AssetDatas)
	{
		AssetPaths.Add(AssetData.GetSoftObjectPath());
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_RemoveAssetsFromCollection(Name, AssetPaths);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UAssetTagsSubsystem::RemoveAssetPtrsFromCollection(const FName Name, const TArray<UObject*>& AssetPtrs)
{
	TArray<FSoftObjectPath> AssetPaths;
	AssetPaths.Reserve(AssetPtrs.Num());
	for (const UObject* AssetPtr : AssetPtrs)
	{
		AssetPaths.Add(FSoftObjectPath(AssetPtr));
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return K2_RemoveAssetsFromCollection(Name, AssetPaths);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif	// WITH_EDITOR

bool UAssetTagsSubsystem::CollectionExists(const FName Name)
{
	bool bExists = false;

#if WITH_EDITOR
	{
		if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
		{
			const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

			TArray<FCollectionScriptingRef> FoundCollections;
			bExists = Subsystem->GetCollectionsByName(FCollectionScriptingContainerSource{ CollectionContainer->GetCollectionSource()->GetName(), CollectionContainer->GetCollectionSource()->GetTitle() }, Name, FoundCollections);
		}
	}
#else	// WITH_EDITOR
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

		const FString TagNameStr = FString::Printf(TEXT("%s%s"), FAssetData::GetCollectionTagPrefix(), *Name.ToString());
		bExists = AssetRegistry.ContainsTag(*TagNameStr);
	}
#endif	// WITH_EDITOR

	return bExists;
}

TArray<FName> UAssetTagsSubsystem::GetCollections()
{
	TArray<FName> CollectionNames;

#if WITH_EDITOR
	{
		if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
		{
			const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

			TArray<FCollectionScriptingRef> Collections;
			if (Subsystem->GetCollections(FCollectionScriptingContainerSource{ CollectionContainer->GetCollectionSource()->GetName(), CollectionContainer->GetCollectionSource()->GetTitle() }, Collections))
			{
				CollectionNames.Reserve(Collections.Num());
				for (const FCollectionScriptingRef& Collection : Collections)
				{
					CollectionNames.AddUnique(Collection.Name);
				}
			}
		}
	}
#else	// WITH_EDITOR
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

		FStringView CollectionTagPrefix(FAssetData::GetCollectionTagPrefix());

		FString TagNameStr;
		AssetRegistry.ReadLockEnumerateAllTagToAssetDatas(
			[&TagNameStr, &CollectionTagPrefix, &CollectionNames](FName TagName, IAssetRegistry::FEnumerateAssetDatasFunc EnumerateAssets)
			{
				TagName.ToString(TagNameStr);
				if (FStringView(TagNameStr).StartsWith(CollectionTagPrefix, ESearchCase::IgnoreCase))
				{
					const FString TrimmedTagNameStr = TagNameStr.Mid(CollectionTagPrefix.Len());
					CollectionNames.Add(*TrimmedTagNameStr);
				}

				return true;
			});
	}
#endif	// WITH_EDITOR

	CollectionNames.Sort(FNameLexicalLess());
	return CollectionNames;
}

TArray<FAssetData> UAssetTagsSubsystem::GetAssetsInCollection(const FName Name)
{
	TArray<FAssetData> Assets;

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
#if WITH_EDITOR
	{
		if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
		{
			const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

			FCollectionScriptingRef ResolvedCollection;
			if (AssetTagsSubsystemUtil::FindCollectionByName(*CollectionContainer, Name, ResolvedCollection))
			{
				Subsystem->GetAssetsInCollection(ResolvedCollection, Assets);
			}
		}
	}
#else	// WITH_EDITOR
	{
		const FString TagNameStr = FString::Printf(TEXT("%s%s"), FAssetData::GetCollectionTagPrefix(), *Name.ToString());

		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = true; // Collection tags are added at cook-time, so we *must* search the on-disk version of the tags (from the asset registry)
		Filter.TagsAndValues.Add(*TagNameStr);

		AssetRegistry.GetAssets(Filter, Assets);
	}
#endif	// WITH_EDITOR

	Assets.Sort();
	return Assets;
}

TArray<FName> UAssetTagsSubsystem::K2_GetCollectionsContainingAsset(const FSoftObjectPath& AssetPath)
{
	TArray<FName> CollectionNames;

#if WITH_EDITOR
	{
		if (UCollectionManagerScriptingSubsystem* Subsystem = GEditor->GetEditorSubsystem<UCollectionManagerScriptingSubsystem>(); ensure(Subsystem))
		{
			const TSharedRef<ICollectionContainer>& CollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

			TArray<FCollectionScriptingRef> Collections;
			if (Subsystem->GetCollectionsContainingAsset(FCollectionScriptingContainerSource{ CollectionContainer->GetCollectionSource()->GetName(), CollectionContainer->GetCollectionSource()->GetTitle() }, AssetPath, Collections))
			{
				CollectionNames.Reserve(Collections.Num());
				for (const FCollectionScriptingRef& Collection : Collections)
				{
					CollectionNames.Add(Collection.Name);
				}
			}
		}
	}
#else	// WITH_EDITOR
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

		const bool bIncludeOnlyOnDiskAssets = true; // Collection tags are added at cook-time, so we *must* search the on-disk version of the tags (from the asset registry)
		const FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(AssetPath, bIncludeOnlyOnDiskAssets);
		if (AssetData.IsValid())
		{
			const TCHAR* CollectionTagPrefix = FAssetData::GetCollectionTagPrefix();
			const int32 CollectionTagPrefixLen = FCString::Strlen(CollectionTagPrefix);

			for (const auto& TagAndValuePair : AssetData.TagsAndValues)
			{
				const FString TagNameStr = TagAndValuePair.Key.ToString();
				if (TagNameStr.StartsWith(CollectionTagPrefix))
				{
					const FString TrimmedTagNameStr = TagNameStr.Mid(CollectionTagPrefixLen);
					CollectionNames.Add(*TrimmedTagNameStr);
				}
			}
		}
	}
#endif	// WITH_EDITOR

	CollectionNames.Sort(FNameLexicalLess());
	return CollectionNames;
}

TArray<FName> UAssetTagsSubsystem::GetCollectionsContainingAsset(const FName AssetPathName)
{
	return K2_GetCollectionsContainingAsset(FSoftObjectPath(AssetPathName.ToString()));
}

TArray<FName> UAssetTagsSubsystem::GetCollectionsContainingAssetData(const FAssetData& AssetData)
{
	// Note: Use the path version as the common implementation as:
	//  1) The path is always required for the collection manager implementation
	//  2) The FAssetData for the asset registry implementation *must* come from the asset registry (as the tags are added at cook-time, and missing if FAssetData is generated from a UObject* at runtime)
	return K2_GetCollectionsContainingAsset(AssetData.GetSoftObjectPath());
}

TArray<FName> UAssetTagsSubsystem::GetCollectionsContainingAssetPtr(const UObject* AssetPtr)
{
	// Note: Use the path version as the common implementation as:
	//  1) The path is always required for the collection manager implementation
	//  2) The FAssetData for the asset registry implementation *must* come from the asset registry (as the tags are added at cook-time, and missing if FAssetData is generated from a UObject* at runtime)
	return K2_GetCollectionsContainingAsset(FSoftObjectPath(AssetPtr));
}

