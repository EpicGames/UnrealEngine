// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerScriptingTypes.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/Package.h"
#include "AssetTagsSubsystem.generated.h"

#define UE_API ASSETTAGS_API

UCLASS(MinimalAPI)
class UAssetTagsSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/**
	 * Create a new collection with the given name and share type.
	 *
	 * @param Name Name to give to the collection.
	 * @param ShareType Whether the collection should be local, private, or shared?
	 *
	 * @return True if the collection was created, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool CreateCollection(const FName Name, const ECollectionScriptingShareType ShareType);
	
	/**
	 * Destroy the given collection.
	 *
	 * @param Name Name of the collection to destroy.
	 *
	 * @return True if the collection was destroyed, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool DestroyCollection(const FName Name);
	
	/**
	 * Rename the given collection.
	 *
	 * @param Name Name of the collection to rename.
	 * @param NewName Name to give to the collection.
	 *
	 * @return True if the collection was renamed, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool RenameCollection(const FName Name, const FName NewName);
	
	/**
	 * Re-parent the given collection.
	 *
	 * @param Name Name of the collection to re-parent.
	 * @param NewParentName Name of the new parent collection, or None to have the collection become a root collection.
	 *
	 * @return True if the collection was renamed, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool ReparentCollection(const FName Name, const FName NewParentName);

	/**
	 * Remove all assets from the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool EmptyCollection(const FName Name);

	/**
	 * Add the given asset to the given collection.
	 * 
	 * @param Name Name of the collection to modify.
	 * @param AssetPathName Asset to add (its path name, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", DisplayName="Add Asset To Collection", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool K2_AddAssetToCollection(const FName Name, const FSoftObjectPath& AssetPath);

	UE_DEPRECATED(5.1, "Names containing full asset paths are deprecated. Use Soft Object Path instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta = (DeprecatedFunction, DeprecationMessage = "Names containing full asset paths are deprecated. Use Soft Object Path instead."))
	UE_API bool AddAssetToCollection(const FName Name, const FName AssetPathName);

	/**
	 * Add the given asset to the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetData Asset to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool AddAssetDataToCollection(const FName Name, const FAssetData& AssetData);

	/**
	 * Add the given asset to the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetPtr Asset to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool AddAssetPtrToCollection(const FName Name, const UObject* AssetPtr);
	
	/**
	 * Add the given assets to the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetPathNames Assets to add (their path names, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", DisplayName="Add Assets To Collection", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool K2_AddAssetsToCollection(const FName Name, const TArray<FSoftObjectPath>& AssetPaths);
	 
	UE_DEPRECATED(5.1, "Names containing full asset paths are deprecated. Use Soft Object Path instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta = (DeprecatedFunction, DeprecationMessage = "Names containing full asset paths are deprecated. Use Soft Object Path instead."))
	UE_API bool AddAssetsToCollection(const FName Name, const TArray<FName>& AssetPathNames);

	/**
	 * Add the given assets to the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetDatas Assets to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool AddAssetDatasToCollection(const FName Name, const TArray<FAssetData>& AssetDatas);

	/**
	 * Add the given assets to the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetPtrs Assets to add.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool AddAssetPtrsToCollection(const FName Name, const TArray<UObject*>& AssetPtrs);

	/**
	 * Remove the given asset from the given collection.
	 * 
	 * @param Name Name of the collection to modify.
	 * @param AssetPath Asset to remove (its path, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", DisplayName="Remove Asset From Collection", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool K2_RemoveAssetFromCollection(const FName Name, const FSoftObjectPath& AssetPath);

	UE_DEPRECATED(5.1, "Names containing full asset paths are deprecated, use Soft Object Path instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta = (DeprecatedFunction, DeprecationMessage = "Names containing full asset paths are deprecated. Use Soft Object Path instead."))
	UE_API bool RemoveAssetFromCollection(const FName Name, const FName AssetPathName);

	/**
	 * Remove the given asset from the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetData Asset to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool RemoveAssetDataFromCollection(const FName Name, const FAssetData& AssetData);

	/**
	 * Remove the given asset from the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetPtr Asset to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool RemoveAssetPtrFromCollection(const FName Name, const UObject* AssetPtr);
	
	/**
	 * Remove the given assets from the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetPathNames Assets to remove (their path names, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", DisplayName="Remove Assets From Collection", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool K2_RemoveAssetsFromCollection(const FName Name, const TArray<FSoftObjectPath>& AssetPaths);

	UE_DEPRECATED(5.1, "Names containing full asset paths are deprecated. Use Soft Object Path instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta = (DeprecatedFunction, DeprecationMessage = "Names containing full asset paths are deprecated. Use Soft Object Path instead."))
	UE_API bool RemoveAssetsFromCollection(const FName Name, const TArray<FName>& AssetPathNames);

	/**
	 * Remove the given assets from the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetDatas Assets to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool RemoveAssetDatasFromCollection(const FName Name, const TArray<FAssetData>& AssetDatas);

	/**
	 * Remove the given assets from the given collection.
	 *
	 * @param Name Name of the collection to modify.
	 * @param AssetPtrs Assets to remove.
	 *
	 * @return True if the collection was modified, false otherwise (see the output log for details on error).
	 */
	UE_DEPRECATED(5.6, "Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta=(DeprecatedFunction, DeprecationMessage="Collection Manager Scripting Subsystem has been introduced as a new top level object for managing collections. Use that instead."))
	UE_API bool RemoveAssetPtrsFromCollection(const FName Name, const TArray<UObject*>& AssetPtrs);
#endif 	// WITH_EDITOR

	/**
	 * Check whether the given collection exists.
	 * Use this for in-game access. Use the Collections Manager Scripting Subsystem for any other use case.
	 * 
	 * @param Name Name of the collection to test.
	 *
	 * @return True if the collection exists, false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category="AssetTags")
	UE_API bool CollectionExists(const FName Name);

	/**
	 * Get the names of all available collections.
	 * Use this for in-game access. Use the Collections Manager Scripting Subsystem for any other use case.
	 * 
	 * @return Names of all available collections.
	 */
	UFUNCTION(BlueprintCallable, Category="AssetTags")
	UE_API TArray<FName> GetCollections();

	/**
	 * Get the assets in the given collection.
	 * Use this for in-game access. Use the Collections Manager Scripting Subsystem for any other use case.
	 * 
	 * @param Name Name of the collection to test.
	 *
	 * @return Assets in the given collection.
	 */
	UFUNCTION(BlueprintCallable, Category="AssetTags")
	UE_API TArray<FAssetData> GetAssetsInCollection(const FName Name);

	/**
	 * Get the names of the collections that contain the given asset.
	 * Use this for in-game access. Use the Collections Manager Scripting Subsystem for any other use case.
	 * 
	 * @param AssetPathName Asset to test (its path name, eg) /Game/MyFolder/MyAsset.MyAsset).
	 *
	 * @return Names of the collections that contain the asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetTags", DisplayName="Get Collections Containing Asset")
	UE_API TArray<FName> K2_GetCollectionsContainingAsset(const FSoftObjectPath& AssetPath);

	UE_DEPRECATED(5.1, "Names containing full asset paths are deprecated. Use Soft Object Path instead.")
	UFUNCTION(BlueprintCallable, Category="AssetTags", meta = (DeprecatedFunction, DeprecationMessage = "Names containing full asset paths are deprecated. Use Soft Object Path instead."))
	UE_API TArray<FName> GetCollectionsContainingAsset(const FName AssetPathName);

	/**
	 * Get the names of the collections that contain the given asset.
	 * Use this for in-game access. Use the Collections Manager Scripting Subsystem for any other use case.
	 * 
	 * @param AssetData Asset to test.
	 *
	 * @return Names of the collections that contain the asset.
	 */
	UFUNCTION(BlueprintCallable, Category="AssetTags")
	UE_API TArray<FName> GetCollectionsContainingAssetData(const FAssetData& AssetData);

	/**
	 * Get the names of the collections that contain the given asset.
	 * Use this for in-game access. Use the Collections Manager Scripting Subsystem for any other use case.
	 * 
	 * @param AssetPtr Asset to test.
	 *
	 * @return Names of the collections that contain the asset.
	 */
	UFUNCTION(BlueprintCallable, Category="AssetTags")
	UE_API TArray<FName> GetCollectionsContainingAssetPtr(const UObject* AssetPtr);
};

#undef UE_API
